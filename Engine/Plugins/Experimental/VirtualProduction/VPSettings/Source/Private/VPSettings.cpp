// Copyright Epic Games, Inc. All Rights Reserved.

#include "VPSettings.h"

const FGameplayTagContainer& UVPSettings::GetRoles() const
{
	static const FGameplayTagContainer EmptyContainer;
	if (ensure(RolesGetter.IsBound()))
	{
		return RolesGetter.Execute();
	}
	
	return EmptyContainer;
}

UVPSettings* UVPSettings::GetVPSettings()
{
	return GetMutableDefault<UVPSettings>();
}
