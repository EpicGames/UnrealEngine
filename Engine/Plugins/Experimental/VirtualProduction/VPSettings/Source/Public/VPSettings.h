// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameplayTagContainer.h"
#include "UObject/Object.h"
#include "Containers/Array.h"
#include "VPSettings.generated.h"

/**
 *
 */
UCLASS(config=Game)
class VPSETTINGS_API UVPSettings : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.1, "Use UVirtualProductionRolesSubsystem::GetCurrentRoles() instead.")
	const FGameplayTagContainer& GetRoles() const;

	/** Default Kit of Focal Lengths for Virtual Camera */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "VirtualCamera|Presets")
	TArray<float> FocalLengthPresets = { 18.0,21.0,25.0,32.0,40.0,50.0,65.0,75.0,100.0,135.0 };

	/** Default Apertures for for Virtual Camera  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "VirtualCamera|Presets")
	TArray<float> AperturePresets = { 1.0f,1.4f,2.0f,2.8f,4.0f,5.6f,8.0f,11.0f,16.0f,22.0f };

	/** Default Shutter Speeds (1/s) for Virtual Camera*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "VirtualCamera|Presets")
	TArray<float> DefaultShutterSpeedPresets = { 1.0,4.0,8.0,15.0,30.0,60.0,125.0,250.0,500.0,1000.0 };

	/** Default ISOs for Virtual Camera*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "VirtualCamera|Presets")
	TArray<float> DefaultISOPresets = { 50,100,200,400,800,1600,3200,6400 };

	/** Default Name for Director*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Virtual Production", DisplayName = "Director Name")
	FString DirectorName = "Default Director";

	/** Default Name for Project*/
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = "Virtual Production", DisplayName = "Project Name")
	FString ShowName = "Default Project";

	UFUNCTION(BlueprintPure, Category = "Virtual Production")
	static UVPSettings* GetVPSettings();

#if WITH_EDITORONLY_DATA

	/** When enabled, the virtual production role(s) will be displayed in the main editor UI. */
	UPROPERTY(config, EditAnywhere, Category="Virtual Production")
	bool bShowRoleInEditor;

	UE_DEPRECATED(5.1, "Use UVirtualProductionRolesSubsystem::OnRolesChanged instead.")
	/** Notify when the virtual production roles have changed. */
	FSimpleMulticastDelegate OnRolesChanged;

#endif
	
protected:
	/**
	 * The machine role(s) in a virtual production context.
	 * @note The role may be override via the command line, "-VPRole=[Role.SubRole1|Role.SubRole2]"
	 */
	UPROPERTY(config)
	FGameplayTagContainer Roles;

private:
	/**
	 * Roles getter set by the VP Roles module to maintain backwards compatibility.
	 * May return the command line roles or roles enabled through the VP Roles subsystem.
	 */
	DECLARE_DELEGATE_RetVal(const FGameplayTagContainer&, FOnGetRoles);
	FOnGetRoles RolesGetter;

	friend class UVirtualProductionRolesSubsystem;
};
