// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VPSettings : ModuleRules
{
	public VPSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
				"CoreUObject",
				"GameplayTags",
			}
		);
	}
}
