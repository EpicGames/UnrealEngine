{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Virtual Production Settings",
	"Description": "Allows users to manage Virtual Production settings.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Hidden" : true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "VPSettings",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
	]
}
